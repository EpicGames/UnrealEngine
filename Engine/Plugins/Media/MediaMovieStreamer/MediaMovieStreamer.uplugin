{
	"FileVersion": 3,
	"Version": 0,
	"VersionName": "0.1",
	"FriendlyName": "Media Movie Streamer",
	"Description": "Movie Streamer using MediaFramework.",
	"Category": "Movie Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Engine/MediaFramework/Overview",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"EnabledByDefault": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "MediaMovieStreamer",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}