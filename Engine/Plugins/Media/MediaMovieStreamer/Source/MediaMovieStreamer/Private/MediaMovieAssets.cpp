// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaMovieAssets.h"
#include "MediaMovieStreamer.h"
#include "MediaPlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaMovieAssets)

UMediaMovieAssets::UMediaMovieAssets()
	: MediaPlayer(nullptr)
	, MediaSoundComponent(nullptr)
	, MediaSource(nullptr)
	, MediaTexture(nullptr)
	, MovieStreamer(nullptr)
{
}

UMediaMovieAssets::~UMediaMovieAssets()
{
}

void UMediaMovieAssets::SetMediaPlayer(UMediaPlayer* InMediaPlayer, FMediaMovieStreamer* InMovieStreamer)
{
	// Unbind from previous player.
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->OnEndReached.RemoveDynamic(this, &UMediaMovieAssets::OnMediaEnd);
	}

	MediaPlayer = InMediaPlayer;
	MovieStreamer = InMovieStreamer;
	
	// Bind to new player.
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->OnEndReached.AddUniqueDynamic(this, &UMediaMovieAssets::OnMediaEnd);
	}
}

void UMediaMovieAssets::SetMediaSoundComponent(UMediaSoundComponent* InMediaSoundComponent)
{
	MediaSoundComponent = InMediaSoundComponent;
}

void UMediaMovieAssets::SetMediaSource(UMediaSource* InMediaSource)
{
	MediaSource = InMediaSource;
}

void UMediaMovieAssets::SetMediaTexture(UMediaTexture* InMediaTexture)
{
	MediaTexture = InMediaTexture;
}

void UMediaMovieAssets::OnMediaEnd()
{
	if (MovieStreamer != nullptr)
	{
		MovieStreamer->OnMediaEnd();
	}
}


