// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MediaMovieAssets.generated.h"

class FMediaMovieStreamer;
class UMediaPlayer;
class UMediaSoundComponent;
class UMediaSource;
class UMediaTexture;

/**
 * Keeps assets alive during level loading so they don't get garbage collected while we are using them.
 * Also handles other UObject functionality like hooking into the UMediaPlayer callbacks
 * which require a UObject.
 */
UCLASS(Transient)
class UMediaMovieAssets : public UObject
{
	GENERATED_BODY()

public:
	UMediaMovieAssets();
	~UMediaMovieAssets();

	/**
	 * Sets which media player we are using.
	 *
	 * @param InMediaPlayer
	 * @param InMovieStreamer Movie streamer that is using the media player.
	 */
	void SetMediaPlayer(UMediaPlayer* InMediaPlayer, FMediaMovieStreamer* InMovieStreamer);

	/**
	 * Sets what media sound component we are using.
	 *
	 * @param InMediaSoundComponent Media sound component to use.
	 */
	void SetMediaSoundComponent(UMediaSoundComponent* InMediaSoundComponent);

	/**
	 * Sets what media source we are using.
	 *
	 * @param InMediaSource Media source to play.
	 */
	void SetMediaSource(UMediaSource* InMediaSource);

	/**
	 * Sets what media texture we are using.
	 *
	 * @param InMediaTexture Media texture to use.
	 */
	void SetMediaTexture(UMediaTexture* InMediaTexture);
	
private:
	/**
	 * Called by the media player when the video ends.
	 */
	UFUNCTION()
	void OnMediaEnd();

	/** Holds the player we are using. */
	UPROPERTY()
	TObjectPtr<UMediaPlayer> MediaPlayer;

	/** Holds the media sound component we are using. */
	UPROPERTY()
	TObjectPtr<UMediaSoundComponent> MediaSoundComponent;

	/** Holds the media source we are using. */
	UPROPERTY()
	TObjectPtr<UMediaSource> MediaSource;

	/** Holds the media texture we are using. */
	UPROPERTY()
	TObjectPtr<UMediaTexture> MediaTexture;

	/** Holds the movie streamer. */
	FMediaMovieStreamer* MovieStreamer;
};
