// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaMovieStreamer.h"

#include "IMediaModule.h"
#include "MediaMovieAssets.h"
#include "MediaMovieStreamerModule.h"
#include "MediaMovieTimeSource.h"
#include "MediaPlayer.h"
#include "MediaSoundComponent.h"
#include "MediaSource.h"
#include "MediaTexture.h"

#include "Modules/ModuleManager.h"
#include "RenderingThread.h"

DEFINE_LOG_CATEGORY(LogMediaMovieStreamer);

FMediaMovieStreamer::FMediaMovieStreamer()
	: bIsPlaying(false)
	, bIsMediaControlledExternally(false)
{
	MovieViewport = MakeShareable(new FMovieViewport());
}

FMediaMovieStreamer::~FMediaMovieStreamer()
{
	Cleanup();
}

void FMediaMovieStreamer::SetIsMediaControlledExternally(bool bInIsMediaControlledExternally)
{
	bIsMediaControlledExternally = bInIsMediaControlledExternally;
}

void FMediaMovieStreamer::SetMediaPlayer(UMediaPlayer* InMediaPlayer)
{
	// Tell MovieAssets about this so it does not get garbage collected.
	UMediaMovieAssets* MovieAssets = FMediaMovieStreamerModule::GetMovieAssets();
	if (MovieAssets != nullptr)
	{
		MovieAssets->SetMediaPlayer(InMediaPlayer, this);
	}

	MediaPlayer = InMediaPlayer;
}

void FMediaMovieStreamer::SetMediaSoundComponent(UMediaSoundComponent* InMediaSoundComponent)
{
	// Tell MovieAssets about this so it does not get garbage collected.
	UMediaMovieAssets* MovieAssets = FMediaMovieStreamerModule::GetMovieAssets();
	if (MovieAssets != nullptr)
	{
		MovieAssets->SetMediaSoundComponent(InMediaSoundComponent);
	}

	MediaSoundComponent = InMediaSoundComponent;
}

void FMediaMovieStreamer::SetMediaSource(UMediaSource* InMediaSource)
{
	// Tell MovieAssets about this so it does not get garbage collected.
	UMediaMovieAssets* MovieAssets = FMediaMovieStreamerModule::GetMovieAssets();
	if (MovieAssets != nullptr)
	{
		MovieAssets->SetMediaSource(InMediaSource);
	}

	MediaSource = InMediaSource;
}

void FMediaMovieStreamer::SetMediaTexture(UMediaTexture* InMediaTexture)
{
	// Tell MovieAssets about this so it does not get garbage collected.
	UMediaMovieAssets* MovieAssets = FMediaMovieStreamerModule::GetMovieAssets();
	if (MovieAssets != nullptr)
	{
		MovieAssets->SetMediaTexture(InMediaTexture);
	}

	MediaTexture = InMediaTexture;
}

void FMediaMovieStreamer::OnMediaEnd()
{
	// Do we control the media?
	if (bIsMediaControlledExternally == false)
	{
		bIsPlaying = false;
	}
}

bool FMediaMovieStreamer::Init(const TArray<FString>& InMoviePaths, TEnumAsByte<EMoviePlaybackType> InPlaybackType)
{
	MovieViewport->SetTexture(nullptr);
	
	// Do we control the media?
	if (bIsMediaControlledExternally == false)
	{
		// Play source.
		if ((MediaPlayer.IsValid()) && (MediaSource.IsValid()))
		{
			bIsPlaying = true;
			MediaPlayer->OpenSource(MediaSource.Get());

			Texture = MakeShareable(new FSlateTexture2DRHIRef(nullptr, 0, 0));
		}
	}
	
	// Set time source as the normal one does not update when the game thread is blocked.
	IMediaModule* MediaModule = GetMediaModule();
	if (MediaModule != nullptr)
	{
		PreviousTimeSource = MediaModule->GetTimeSource();
		MediaModule->SetTimeSource(MakeShareable(new FMediaMovieTimeSource));
	}

	return true;
}

void FMediaMovieStreamer::PreviousViewportInterface(const TSharedPtr<ISlateViewport>& PreviousViewportInterface)
{
	// Use the size of the previous viewport as our default,
	// as we may not have a size ourselves if we don't have a texture.
	FIntPoint ViewportSize(ForceInitToZero);
	if (PreviousViewportInterface.IsValid())
	{
		ViewportSize = PreviousViewportInterface->GetSize();
	}

	MovieViewport->SetDefaultSize(ViewportSize);
}

void FMediaMovieStreamer::ForceCompletion()
{
}

bool FMediaMovieStreamer::Tick(FRHICommandListBase& RHICmdList, float DeltaTime)
{
	check(IsInRenderingThread());

	// Do we control the media?
	if (bIsMediaControlledExternally == false)
	{
		// Get media texture.
		if (MediaTexture != nullptr)
		{
			FTextureResource* TextureResource = MediaTexture->GetResource();
			if (TextureResource != nullptr)
			{
				FRHITexture* RHITexture2D = TextureResource->GetTexture2DRHI();
				if (RHITexture2D != nullptr)
				{
					// Get slate texture.
					FSlateTexture2DRHIRef* CurrentTexture = Texture.Get();
					if (CurrentTexture)
					{
						if (!CurrentTexture->IsInitialized())
						{
							CurrentTexture->InitResource(RHICmdList);
						}

						// Update the slate texture.
						FTextureRHIRef ref = RHITexture2D;
						CurrentTexture->SetRHIRef(ref, MediaTexture->GetWidth(), MediaTexture->GetHeight());

						// Update viewport.
						if (MovieViewport->GetViewportRenderTargetTexture() == nullptr)
						{
							MovieViewport->SetTexture(Texture);
						}
					}
				}
			}
		}
	}

	return !bIsPlaying;
}

TSharedPtr<class ISlateViewport> FMediaMovieStreamer::GetViewportInterface()
{
	return MovieViewport;
}

float FMediaMovieStreamer::GetAspectRatio() const
{
	return 1.0f;
}

FString FMediaMovieStreamer::GetMovieName()
{
	return FString();
}

bool FMediaMovieStreamer::IsLastMovieInPlaylist()
{
	return true;
}

void FMediaMovieStreamer::Cleanup()
{
	// Do we control the media?
	if (bIsMediaControlledExternally == false && !IsEngineExitRequested())
	{
		// Remove our hold on the assets.
		UMediaMovieAssets* MovieAssets = FMediaMovieStreamerModule::GetMovieAssets();
		if (MovieAssets != nullptr)
		{
			MovieAssets->SetMediaPlayer(nullptr, nullptr);
			MovieAssets->SetMediaSoundComponent(nullptr);
			MovieAssets->SetMediaSource(nullptr);
		}

		MediaPlayer.Reset();
		MediaSoundComponent.Reset();
		MediaSource.Reset();

		MovieViewport->SetTexture(NULL);

		FSlateTexture2DRHIRef* CurrentTexture = Texture.Get();
		if (CurrentTexture != nullptr)
		{
			// Remove any reference to UMediaTexture so we can let MediaFramework release it.
			ENQUEUE_RENDER_COMMAND(ResetMovieTexture)(
				[CurrentTexture](FRHICommandListImmediate& RHICmdList)
			{
				CurrentTexture->SetRHIRef(nullptr, 0, 0);
			});
			BeginReleaseResource(CurrentTexture);
			FlushRenderingCommands();
			Texture.Reset();
		}
	}

	// Restore previous time source.
	IMediaModule* MediaModule = GetMediaModule();
	if ((MediaModule != nullptr) && (PreviousTimeSource != nullptr))
	{
		MediaModule->SetTimeSource(PreviousTimeSource);
	}
}

FTextureRHIRef FMediaMovieStreamer::GetTexture()
{
	return nullptr;
}

FMediaMovieStreamer::FOnCurrentMovieClipFinished& FMediaMovieStreamer::OnCurrentMovieClipFinished()
{
	return OnCurrentMovieClipFinishedDelegate;
}

void FMediaMovieStreamer::TickPreEngine()
{
	IMediaModule* MediaModule = GetMediaModule();
	if (MediaModule != nullptr)
	{
		MediaModule->TickPreEngine();
	}
	if (MovieStreamerPreEngineTick.IsBound())
	{
		MovieStreamerPreEngineTick.Broadcast();
	}
}

void FMediaMovieStreamer::TickPostEngine()
{
	IMediaModule* MediaModule = GetMediaModule();
	if (MediaModule != nullptr)
	{
		MediaModule->TickPostEngine();
	}
	if (MovieStreamerPostEngineTick.IsBound())
	{
		MovieStreamerPostEngineTick.Broadcast();
	}
}

void FMediaMovieStreamer::TickPostRender()
{
	IMediaModule* MediaModule = GetMediaModule();
	if (MediaModule != nullptr)
	{
		MediaModule->TickPostRender();
	}
	if (MovieStreamerPostRenderTick.IsBound())
	{
		MovieStreamerPostRenderTick.Broadcast();
	}
}

IMediaModule* FMediaMovieStreamer::GetMediaModule()
{
	static const FName MediaModuleName(TEXT("Media"));
	IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);
	return MediaModule;
}

