// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaMovieStreamerModule.h"

#include "MediaMovieAssets.h"
#include "MediaMovieStreamer.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FMediaMovieStreamerModule"

TSharedPtr<FMediaMovieStreamer, ESPMode::ThreadSafe> MovieStreamer;
TWeakObjectPtr<UMediaMovieAssets> MovieAssets;
bool FMediaMovieStreamerModule::bStartedModule = false;

UMediaMovieAssets* FMediaMovieStreamerModule::GetMovieAssets()
{
	if ((MovieAssets.IsValid() == false) && bStartedModule)
	{
		// Create MovieAssets.
		MovieAssets = NewObject<UMediaMovieAssets>();
		MovieAssets->AddToRoot();
	}

	return MovieAssets.Get();
}

const TSharedPtr<FMediaMovieStreamer, ESPMode::ThreadSafe> FMediaMovieStreamerModule::GetMovieStreamer()
{
	if (!MovieStreamer.IsValid() && bStartedModule)
	{
		// Create MovieStreamer.
		MovieStreamer = MakeShareable(new FMediaMovieStreamer);
		FCoreDelegates::RegisterMovieStreamerDelegate.Broadcast(MovieStreamer);
	}

	return MovieStreamer;
}

void FMediaMovieStreamerModule::StartupModule()
{
	bStartedModule = true;
}

void FMediaMovieStreamerModule::ShutdownModule()
{
	bStartedModule = false;

	// Shutdown MovieStreamer.
	if (MovieStreamer.IsValid())
	{
		FCoreDelegates::UnRegisterMovieStreamerDelegate.Broadcast(MovieStreamer);
		MovieStreamer.Reset();
	}

	// Shutdown MovieAssets.
	if (MovieAssets.IsValid())
	{
		MovieAssets->RemoveFromRoot();
		MovieAssets.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FMediaMovieStreamerModule, MediaMovieStreamer)
