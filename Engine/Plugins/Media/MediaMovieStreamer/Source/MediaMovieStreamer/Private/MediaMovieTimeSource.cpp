// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaMovieTimeSource.h"

#include "HAL/PlatformTime.h"

FTimespan FMediaMovieTimeSource::GetTimecode()
{
	const double CurrentTime = FPlatformTime::Seconds();
	const FTimespan Timecode = FTimespan::FromSeconds(CurrentTime);

	return Timecode;
}
