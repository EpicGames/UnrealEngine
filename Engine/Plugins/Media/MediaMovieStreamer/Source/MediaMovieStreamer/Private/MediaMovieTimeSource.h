// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaTimeSource.h"

struct FTimespan;


/**
 * Implements the a media time source that derives its time from the application's global time.
 */
class FMediaMovieTimeSource
	: public IMediaTimeSource
{
public:

	/** Virtual destructor. */
	virtual ~FMediaMovieTimeSource() = default;

public:

	//~ IMediaTimeSource interface

	virtual FTimespan GetTimecode() override;
};
