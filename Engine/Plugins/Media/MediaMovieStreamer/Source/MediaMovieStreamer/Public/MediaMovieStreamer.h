// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePlayer.h"

class FSlateTexture2DRHIRef;
class ISlateViewport;

DECLARE_LOG_CATEGORY_EXTERN(LogMediaMovieStreamer, Log, All);

class IMediaModule;
class IMediaTimeSource;
class UMediaPlayer;
class UMediaSoundComponent;
class UMediaSource;
class UMediaTexture;

DECLARE_EVENT(FMediaMovieStreamer, FMovieStreamerExternalTick);

/**
 * Movie streamer that allows you to use MediaFramework during level loading, etc.
 */
class FMediaMovieStreamer : public IMovieStreamer
{
public:
	FMediaMovieStreamer();
	~FMediaMovieStreamer();

	/**
	 * If you do not want FMediaMovieStreamer to play automatically,
	 * but want to control the media yourself, then call this with true.
	 * 
	 * This also means that FMediaMovieStreamer will no longer clean up the assets when it is done.
	 * You will have to call SetMediaPlayer, SetMediaSoundComponent, etc
	 * with nullptr when you are done so FMediaMovieStreamer will no longer keep its hold on them.
	 *
	 * @param bIsMediaControlledExternally True if you want to control the media.
	 */
	MEDIAMOVIESTREAMER_API void SetIsMediaControlledExternally(bool bInIsMediaControlledExternally);

	/**
	 * Sets which media player should be playing.
	 *
	 * @param InMediaPlayer
	 */
	MEDIAMOVIESTREAMER_API void SetMediaPlayer(UMediaPlayer* InMediaPlayer);

	/**
	 * Sets what media sound component we are using.
	 *
	 * @param InMediaSoundComponent Media sound component that is being used.
	 */
	MEDIAMOVIESTREAMER_API void SetMediaSoundComponent(UMediaSoundComponent* InMediaSoundComponent);

	/**
	 * Returns the current MediaSoundComponent
	 */
	UMediaSoundComponent* GetMediaSoundComponent() { return MediaSoundComponent.Get(); }

	/**
	 * Sets what to play.
	 *
	 * @param InMediaSource Media source to play.
	 */
	MEDIAMOVIESTREAMER_API void SetMediaSource(UMediaSource* InMediaSource);

	/**
	 * Sets what media texture we are using.
	 *
	 * @param InMediaSource Media source to play.
	 */
	MEDIAMOVIESTREAMER_API void SetMediaTexture(UMediaTexture* InMediaTexture);

	FMovieStreamerExternalTick MovieStreamerPostEngineTick;
	FMovieStreamerExternalTick MovieStreamerPreEngineTick;
	FMovieStreamerExternalTick MovieStreamerPostRenderTick;

	/**
	 * Called from UMediaMovieAssets when the media ends.
	 */
	void OnMediaEnd();

	/** IMovieStreamer interface */
	virtual bool Init(const TArray<FString>& InMoviePaths, TEnumAsByte<EMoviePlaybackType> InPlaybackType) override;
	virtual void PreviousViewportInterface(const TSharedPtr<ISlateViewport>& PreviousViewportInterface) override;
	virtual void ForceCompletion() override;
	virtual bool Tick(FRHICommandListBase& RHICmdList, float DeltaTime) override;
	virtual TSharedPtr<class ISlateViewport> GetViewportInterface() override;
	virtual float GetAspectRatio() const override;
	virtual FString GetMovieName() override;
	virtual bool IsLastMovieInPlaylist() override;
	virtual void Cleanup() override;
	virtual FTextureRHIRef GetTexture() override;
	virtual FOnCurrentMovieClipFinished& OnCurrentMovieClipFinished() override;

	virtual void TickPreEngine() override;
	virtual void TickPostEngine() override;
	virtual void TickPostRender() override;
	
private:
	/** Delegate for when the movie is finished. */
	FOnCurrentMovieClipFinished OnCurrentMovieClipFinishedDelegate;

	/** Viewport data for displaying to Slate. */
	TSharedPtr<FMovieViewport> MovieViewport;
	/** Texture displaying to Slate. */
	TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> Texture;

	/** Holds the player we are using. */
	TWeakObjectPtr<UMediaPlayer> MediaPlayer;
	/** Holds the media sound component we are using. */
	TWeakObjectPtr<UMediaSoundComponent> MediaSoundComponent;
	/** Holds the media source we are using. */
	TWeakObjectPtr<UMediaSource> MediaSource;
	/** Holds the media texture we are using. */
	TWeakObjectPtr<UMediaTexture> MediaTexture;

	/** Gets the media module to interface with MediaFramework. */
	IMediaModule* GetMediaModule();

	/** True if the media is still playing. */
	bool bIsPlaying;

	/** True if somethiing else is controlling the media. */
	bool bIsMediaControlledExternally;

	/** Stores the previous time source so we can restore it when we are done. */
	TSharedPtr<IMediaTimeSource, ESPMode::ThreadSafe> PreviousTimeSource;
};
