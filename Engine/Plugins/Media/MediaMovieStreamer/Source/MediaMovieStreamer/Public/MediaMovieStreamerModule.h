// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FMediaMovieStreamer;
class UMediaMovieAssets;

class FMediaMovieStreamerModule : public IModuleInterface
{
public:
	/**
	 * Call this to get the MovieAssets.
	 */
	static UMediaMovieAssets* GetMovieAssets();

	/**
	 * Call this to get the MovieStreamer.
	 */
	MEDIAMOVIESTREAMER_API static const TSharedPtr<FMediaMovieStreamer, ESPMode::ThreadSafe> GetMovieStreamer();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	static bool bStartedModule;
};
